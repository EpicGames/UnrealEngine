// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportSnappingModule.h"

#include "Containers/Array.h"
#include "HAL/PlatformCrt.h"
#include "ISnappingPolicy.h"
#include "Math/MathFwd.h"
#include "Math/Rotator.h"

class FPrimitiveDrawInterface;
class FSceneView;

//////////////////////////////////////////////////////////////////////////
// FMergedSnappingPolicy

class FMergedSnappingPolicy : public ISnappingPolicy
{
public:
	TArray< TSharedPtr<ISnappingPolicy> > PolicyList;
public:
	virtual void SnapScale(FVector& Point, const FVector& GridBase) override
	{
		for (auto PolicyIt = PolicyList.CreateConstIterator(); PolicyIt; ++PolicyIt)
		{
			(*PolicyIt)->SnapScale(Point, GridBase);
		}
	}

	virtual void SnapPointToGrid(FVector& Point, const FVector& GridBase) override
	{
		for (auto PolicyIt = PolicyList.CreateConstIterator(); PolicyIt; ++PolicyIt)
		{
			(*PolicyIt)->SnapPointToGrid(Point, GridBase);
		}
	}

	virtual void SnapRotatorToGrid(FRotator& Rotation) override
	{
		for (auto PolicyIt = PolicyList.CreateConstIterator(); PolicyIt; ++PolicyIt)
		{
			(*PolicyIt)->SnapRotatorToGrid(Rotation);
		}
	}

	virtual void ClearSnappingHelpers(bool bClearImmediately) override
	{
		for (auto PolicyIt = PolicyList.CreateConstIterator(); PolicyIt; ++PolicyIt)
		{
			(*PolicyIt)->ClearSnappingHelpers(bClearImmediately);
		}
	}

	virtual void DrawSnappingHelpers(const FSceneView* View, FPrimitiveDrawInterface* PDI) override
	{
		for (auto PolicyIt = PolicyList.CreateConstIterator(); PolicyIt; ++PolicyIt)
		{
			(*PolicyIt)->DrawSnappingHelpers(View, PDI);
		}
	}

	virtual bool IsRotationSnappingEnabled() const override
	{
		for (auto PolicyIt = PolicyList.CreateConstIterator(); PolicyIt; ++PolicyIt)
		{
			if ((*PolicyIt)->IsRotationSnappingEnabled())
			{
				return true;
			}
		}
		return false;
	}

	virtual bool IsTranslationSnappingEnabled() const override
	{
		for (auto PolicyIt = PolicyList.CreateConstIterator(); PolicyIt; ++PolicyIt)
		{
			if ((*PolicyIt)->IsTranslationSnappingEnabled())
			{
				return true;
			}
		}
		return false;
	}

	virtual bool IsScaleSnappingEnabled() const override
	{
		for (auto PolicyIt = PolicyList.CreateConstIterator(); PolicyIt; ++PolicyIt)
		{
			if ((*PolicyIt)->IsScaleSnappingEnabled())
			{
				return true;
			}
		}
		return false;
	}
};

//////////////////////////////////////////////////////////////////////////
// FViewportSnappingModule

class FViewportSnappingModule : public IViewportSnappingModule
{
public:
	TSharedPtr<FMergedSnappingPolicy> MergedPolicy;
public:
	FViewportSnappingModule()
	{
	}

	// IViewportSnappingModule interface
	virtual void RegisterSnappingPolicy(TSharedPtr<ISnappingPolicy> NewPolicy) override
	{
		MergedPolicy->PolicyList.Add(NewPolicy);
	}

	virtual void UnregisterSnappingPolicy(TSharedPtr<ISnappingPolicy> PolicyToRemove) override
	{
		MergedPolicy->PolicyList.Remove(PolicyToRemove);
	}
	
	virtual TSharedPtr<ISnappingPolicy> GetMergedPolicy() override
	{
		return MergedPolicy;
	}

	// End of IViewportSnappingModule interface

	// IModuleInterface interface
	virtual void StartupModule() override
	{
		MergedPolicy = MakeShareable(new FMergedSnappingPolicy);
	}

	virtual void ShutdownModule() override
	{
		MergedPolicy.Reset();
	}
	// End of IModuleInterface interface
};

IMPLEMENT_MODULE( FViewportSnappingModule, ViewportSnapping );
