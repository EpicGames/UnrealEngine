// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FPrimitiveDrawInterface;
class FSceneView;

class ISnappingPolicy
{
public:
	virtual ~ISnappingPolicy() = default;

	/**
	 * Snaps a scale value to the scale grid
	 *
	 * @param Point	The point to snap.  This value will be modified to account for snapping
	 * @param GridBase	Base grid offset
	 */
	virtual void SnapScale(FVector& Point, const FVector& GridBase)=0;

	/**
	 * Snaps a point value to the positional grid
	 *
	 * @param Point	The point to snap.  This value will be modified to account for snapping
	 * @param GridBase	Base grid offset
	 */
	virtual void SnapPointToGrid(FVector& Point, const FVector& GridBase)=0;

	/**
	 * Snaps a rotator to the rotational grid
	 *
	 * @param Rotation	The rotator to snap.  This value will be modified to account for snapping
	 */
	virtual void SnapRotatorToGrid(FRotator& Rotation)=0;

	/**
	 * Clears all vertices being drawn to help a user snap
	 *
	 * @param bClearImmediatley	true to clear helpers without fading them out
	 */
	virtual void ClearSnappingHelpers(bool bClearImmediately = false)=0;

	/**
	 * Draws snapping helpers 
	 *
	 * @param View	The current view of the scene
	 * @param PDI	Drawing interface
	 */
	virtual void DrawSnappingHelpers(const FSceneView* View, FPrimitiveDrawInterface* PDI)=0;

	/**
	 * @return true if rotation snapping is enabled
	 */
	virtual bool IsRotationSnappingEnabled() const = 0;

	/**
	 * @return true if translation snapping is enabled
	 */
	virtual bool IsTranslationSnappingEnabled() const = 0;

	/**
	 * @return true if scale snapping is enabled
	 */
	virtual bool IsScaleSnappingEnabled() const = 0;
};
