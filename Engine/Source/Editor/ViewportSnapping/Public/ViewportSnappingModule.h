// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

class ISnappingPolicy;

/**
 * Snapping policy manager module
 */
class IViewportSnappingModule : public IModuleInterface//, public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	virtual void RegisterSnappingPolicy(TSharedPtr<ISnappingPolicy> NewPolicy)=0;
	virtual void UnregisterSnappingPolicy(TSharedPtr<ISnappingPolicy> PolicyToRemove)=0;

	virtual TSharedPtr<ISnappingPolicy> GetMergedPolicy()=0;

	static TSharedPtr<ISnappingPolicy> GetSnapManager()
	{
		IViewportSnappingModule& Module = FModuleManager::LoadModuleChecked<IViewportSnappingModule>("ViewportSnapping");
		return Module.GetMergedPolicy();
	}
};

