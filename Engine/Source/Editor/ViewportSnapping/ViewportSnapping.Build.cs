// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class ViewportSnapping : ModuleRules
{
	public ViewportSnapping(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("UnrealEd");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
// 				"RenderCore",
// 				"RHI",
			}
			);

// 		DynamicallyLoadedModuleNames.AddRange(
// 			new string[] {
// 				"MainFrame",
// 				"WorkspaceMenuStructure",
// 				"PropertyEditor"
// 			}
// 			);
	}
}
